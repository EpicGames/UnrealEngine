{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "GeometryCacheLevelSequenceBaker",
	"Description": "Bake Skeletal Meshes in Level Sequence to Geometry Cache",
	"Category": "Other",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "GeometryCacheLevelSequenceBaker",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		}
	],
	"Plugins": [
		{
			"Name" : "LevelSequenceEditor",
			"Enabled" : true
		},
		{
			"Name" : "GeometryCache",
			"Enabled" : true
		},
		{
			"Name" : "DeformerGraph",
			"Enabled" : true
		}
	]
}