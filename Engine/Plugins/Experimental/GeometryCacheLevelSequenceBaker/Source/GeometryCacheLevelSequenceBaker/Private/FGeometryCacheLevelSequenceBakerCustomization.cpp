// Copyright Epic Games, Inc. All Rights Reserved.
#include "FGeometryCacheLevelSequenceBakerCustomization.h"

#include "GeometryCacheLevelSequenceBaker.h"
#include "GeometryCacheLevelSequenceBakerCommands.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "ISequencerModule.h"
#include "ISequencer.h"
#include "Modules/ModuleManager.h"
#include "Toolkits/AssetEditorToolkit.h"

void FGeometryCacheLevelSequenceBakerCustomization::RegisterSequencerCustomization(FSequencerCustomizationBuilder& Builder)
{
	WeakSequencer = Builder.GetSequencer().AsShared();

	ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
	
	const FGeometryCacheLevelSequenceBakerCommands& Commands = FGeometryCacheLevelSequenceBakerCommands::Get();
	
	// Build the extender for the actions menu.
	ActionsMenuCommandList = MakeShared<FUICommandList>().ToSharedPtr();
	ActionsMenuCommandList->MapAction(
		Commands.BakeGeometryCache,
		FExecuteAction::CreateRaw( this, &FGeometryCacheLevelSequenceBakerCustomization::BakeGeometryCache),
		FCanExecuteAction::CreateLambda( [] { return true; } ) );
	
	ActionsMenuExtender = MakeShared<FExtender>();
	ActionsMenuExtender->AddMenuExtension(
			"SequenceOptions", EExtensionHook::First, nullptr,
			FMenuExtensionDelegate::CreateRaw(this, &FGeometryCacheLevelSequenceBakerCustomization::ExtendActionsMenu));

	SequencerModule.GetActionsMenuExtensibilityManager()->AddExtender(ActionsMenuExtender);	
}

void FGeometryCacheLevelSequenceBakerCustomization::UnregisterSequencerCustomization()
{
	ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
	SequencerModule.GetActionsMenuExtensibilityManager()->RemoveExtender(ActionsMenuExtender);

	ActionsMenuCommandList = nullptr;
	WeakSequencer = nullptr;
}

void FGeometryCacheLevelSequenceBakerCustomization::ExtendActionsMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.PushCommandList(ActionsMenuCommandList.ToSharedRef());
	{
		const FGeometryCacheLevelSequenceBakerCommands& Commands = FGeometryCacheLevelSequenceBakerCommands::Get();
		
		MenuBuilder.AddMenuEntry(Commands.BakeGeometryCache);
	}
	MenuBuilder.PopCommandList();
}

void FGeometryCacheLevelSequenceBakerCustomization::BakeGeometryCache()
{
	FGeometryCacheLevelSequenceBaker::Bake(WeakSequencer.Pin().ToSharedRef());
}
