// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "SequencerCustomizationManager.h"

class FGeometryCacheLevelSequenceBakerCustomization : public ISequencerCustomization
{
public:
	void RegisterSequencerCustomization(FSequencerCustomizationBuilder& Builder) override;
	void UnregisterSequencerCustomization() override;

	void ExtendActionsMenu(FMenuBuilder& MenuBuilder);
	void BakeGeometryCache();
	
	TWeakPtr<ISequencer> WeakSequencer;

	TSharedPtr<FUICommandList> ActionsMenuCommandList;
	TSharedPtr<FExtender> ActionsMenuExtender;
};
