// Copyright Epic Games, Inc. All Rights Reserved.
#include "GeometryCacheLevelSequenceBaker.h"

#include "Components/SkeletalMeshComponent.h"
#include "AssetToolsModule.h"
#include "EditorDirectories.h"
#include "GeometryCache.h"
#include "GeometryCacheConstantTopologyWriter.h"
#include "GeometryCacheLevelSequenceBakerModule.h"
#include "GeometryCacheTrack.h"
#include "GeometryCacheTrackStreamable.h"
#include "MovieScene.h"
#include "MovieSceneSequencePlayer.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Dialogs/DlgPickAssetPath.h"
#include "Framework/Notifications/NotificationManager.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/Extensions/IObjectBindingExtension.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Engine/SkinnedAsset.h"
#include "Engine/SkeletalMesh.h"
#include "Interfaces/IMainFrameModule.h"
#include "PropertyEditorModule.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "IDetailsView.h"
#include "SequencerSettings.h"
#include "StaticMeshAttributes.h"
#include "Animation/MeshDeformerGeometryReadback.h"
#include "Async/ParallelFor.h"
#include "Materials/MaterialInstanceDynamic.h"


#define LOCTEXT_NAMESPACE "GeometryCacheLevelSequenceBaker"

const int32 FGeometryCacheLevelSequenceBaker::LODIndexToBake = 0;
const float FGeometryCacheLevelSequenceBaker::TotalAmountOfWork = 1.0f;
const float FGeometryCacheLevelSequenceBaker::AmountOfWorkGatherStage = TotalAmountOfWork / 2.0f;
const float FGeometryCacheLevelSequenceBaker::AmountOfWorkBakeStage = TotalAmountOfWork - AmountOfWorkGatherStage;

class SLevelSequenceGeometryCacheBakerOptionWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SLevelSequenceGeometryCacheBakerOptionWindow) {}
		SLATE_ARGUMENT( TSharedPtr<SWindow>, WidgetWindow )
		SLATE_ARGUMENT( ULevelSequenceGeometryCacheBakerOption*, Option)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		WidgetWindow = InArgs._WidgetWindow;
		Option = InArgs._Option;


		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
		DetailsView->SetObject(Option);
		
		this->ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				DetailsView.ToSharedRef()	
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				[
					SAssignNew(BakeButton, SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("LevelSequenceGeometryCacheBakerOptionsWindow_Bake", "Bake"))
					.OnClicked(this, &SLevelSequenceGeometryCacheBakerOptionWindow::OnBake)
				]
				+SHorizontalBox::Slot()
				[
					SAssignNew(CancelButton, SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("LevelSequenceGeometryCacheBakerOptionsWindow_Cancel", "Cancel"))
					.OnClicked(this, &SLevelSequenceGeometryCacheBakerOptionWindow::OnCancel)
				]
			]
		];
	}

	bool SupportsKeyboardFocus() const override { return true; }
	
	FReply OnBake()
	{
		Option->bShouldBake = true;
		if ( WidgetWindow.IsValid() )
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	FReply OnCancel()
	{
		if ( WidgetWindow.IsValid() )
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
	{
		if( InKeyEvent.GetKey() == EKeys::Escape )
		{
			return OnCancel();
		}

		return FReply::Unhandled();
	}

	TWeakPtr<SWindow> WidgetWindow;
	TObjectPtr<ULevelSequenceGeometryCacheBakerOption> Option;
	TSharedPtr<IDetailsView> DetailsView;
	TSharedPtr<SButton> BakeButton;
	TSharedPtr<SButton> CancelButton;
};



void FGeometryCacheLevelSequenceBaker::Bake(TSharedRef<ISequencer> InSequencer)
{
	TSharedPtr<ISequencer> Sequencer = InSequencer;
	UMovieSceneSequence* MovieSceneSequence = Sequencer->GetFocusedMovieSceneSequence();
	UMovieScene* MovieScene = MovieSceneSequence->GetMovieScene();
	
	TArray<FGuid> Bindings = GetBindingsToBake(InSequencer);
	if (Bindings.Num() == 0 )
	{
		return;
	}
	
	FString PackageName;
	FString AssetName;

	if (!GetGeometryCacheAssetPathFromUser(PackageName, AssetName))
	{
		return;
	}

	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(LOCTEXT("LevelSequenceGeometryCacheBakerOptionTitle", "Bake Geometry Cache Options"))
		.SizingRule(ESizingRule::Autosized);

	ULevelSequenceGeometryCacheBakerOption* Option = NewObject<ULevelSequenceGeometryCacheBakerOption>();
	
	TSharedPtr<SLevelSequenceGeometryCacheBakerOptionWindow> OptionWindow;
	Window->SetContent
	(
		SAssignNew(OptionWindow, SLevelSequenceGeometryCacheBakerOptionWindow)
		.WidgetWindow(Window)
		.Option(Option)
	);
	
	GEditor->EditorAddModalWindow(Window);

	if (Option->bShouldBake == false)
	{
		return;
	}

	const TRange<FFrameNumber> PlaybackRange = MovieScene->GetPlaybackRange();
	const FFrameRate Resolution = MovieScene->GetTickResolution();
	const FFrameRate DisplayRate = MovieScene->GetDisplayRate();

	const FFrameTime Interval = FFrameRate::TransformTime(1, DisplayRate, Resolution);
	FFrameTime StartFrame = UE::MovieScene::DiscreteInclusiveLower(PlaybackRange);
	FFrameTime EndFrame = UE::MovieScene::DiscreteExclusiveUpper(PlaybackRange);

	int32 NumFrames = FMath::FloorToInt32((EndFrame.AsDecimal() - StartFrame.AsDecimal()) / Interval.AsDecimal());
	int32 NumSamplesPerFrame = Option->NumSamplesPerFrame;
	int32 TotalSamples = NumFrames * NumSamplesPerFrame;
	// Also capture the last frame since geometry cache use the last frame's time to compute total duration
	TotalSamples += 1;

	

	// Setup bake task
	FGeometryCacheLevelSequenceBaker& Baker = Get();
	check(Baker.CurrentBakeTask == nullptr);
	Baker.CurrentBakeTask =  MakeUnique<FBakeTask>();

	TUniquePtr<FBakeTask>& BakeTask	= Baker.CurrentBakeTask;
	BakeTask->PackageName = PackageName;
	BakeTask->AssetName = AssetName;

	BakeTask->Sequencer = Sequencer;
	BakeTask->Bindings = Bindings;
	BakeTask->StartFrame = StartFrame;
	BakeTask->EndFrame = EndFrame;
	
	BakeTask->NumSamples = TotalSamples;
	BakeTask->CurrentSampleIndex = 0;
	BakeTask->SamplesPerSecond = static_cast<float>(NumSamplesPerFrame * DisplayRate.AsDecimal());

	BakeTask->SlowTask = MakeUnique<FScopedSlowTask>(1.0f, LOCTEXT("BakeGeometryCacheSlowTask", "Baking Geometry Cache..."));
	BakeTask->SlowTask->MakeDialog(true);
	
	double DeltaTime = DisplayRate.AsInterval() / NumSamplesPerFrame;	
	BakeTask->FixedDeltaTimeScope = MakeUnique<FEngineFixedDeltaTimeScope>(DeltaTime);
	BakeTask->SequencerStateScope = MakeUnique<FSequencerSettingScope>(Sequencer);
	BakeTask->ConsoleVariableOverrideScope = MakeUnique<FConsoleVariableOverrideScope>();
	BakeTask->WorldTickEndDelegateScope = MakeUnique<FWorldTickEndDelegateScope>();
}

TArray<FGuid> FGeometryCacheLevelSequenceBaker::GetBindingsToBake(TSharedRef<ISequencer> InSequencer)
{
	TSharedPtr<ISequencer> Sequencer = InSequencer;
	UMovieSceneSequence* MovieSceneSequence = Sequencer->GetFocusedMovieSceneSequence();
	UMovieScene* MovieScene = MovieSceneSequence->GetMovieScene();
	
	using namespace UE::Sequencer;

	// Bake only selected bindings if there are any, otherwise bake every binding
	TArray<FGuid> SelectedBindings;
	TSharedPtr<FSequencerEditorViewModel> EditorViewModel = Sequencer->GetViewModel();
	for (FViewModelPtr Node : EditorViewModel->GetSelection()->Outliner)
	{
		if (TViewModelPtr<IObjectBindingExtension> ObjectBindingNode = Node.ImplicitCast())
		{
			SelectedBindings.Add(ObjectBindingNode->GetObjectGuid());
		}
	}

	TArray<FGuid> CandidateBindings;
	for (const FMovieSceneBinding& MovieSceneBinding : ((const UMovieScene*)MovieScene)->GetBindings())
	{
		// If there are specific bindings to export, export those only
		if (SelectedBindings.Num() != 0 && !SelectedBindings.Contains(MovieSceneBinding.GetObjectGuid()))
		{
			continue;
		}

		CandidateBindings.Add(MovieSceneBinding.GetObjectGuid());
	}
	
	// Skip a child binding if its parent binding is already getting baked
	TArray<FMovieScenePossessable*> Possessables;
	TArray<FMovieSceneSpawnable*> Spawnables;
	for (FGuid Binding : CandidateBindings)
	{
		if (FMovieScenePossessable* Possessable = MovieScene->FindPossessable(Binding))
		{
			Possessables.Add(Possessable);
		}
		else if (FMovieSceneSpawnable* Spawnable = MovieScene->FindSpawnable(Binding))
		{
			Spawnables.Add(Spawnable);
		}
	}

	TArray<FGuid> FinalizedBindings;
	for (FMovieSceneSpawnable* Spawnable : Spawnables)
	{
		FinalizedBindings.Add(Spawnable->GetGuid());
	}
	
	for (FMovieScenePossessable* Possessable : Possessables)
	{
		check(Possessable);
		// Skip possessables that belong to a spawnable that we are already baking
		for (FMovieSceneSpawnable* Spawnable : Spawnables)
		{
			check(Spawnable);
			
			if (Spawnable->GetChildPossessables().Contains(Possessable->GetGuid()))
			{
				continue;
			}
		}

		// Skip possessables that belong to a Possessable that we are already baking
		TArray<FMovieScenePossessable*> PossessableParents;
		FMovieScenePossessable* WorkItem = Possessable;
		while (WorkItem)
		{
			WorkItem = MovieScene->FindPossessable(WorkItem->GetParent());
			if (WorkItem)
			{
				PossessableParents.Add(WorkItem);
			}
		}

		bool bSkip = false;
		for (FMovieScenePossessable* Parent : PossessableParents)
		{
			if (Possessables.Contains(Parent))
			{
				bSkip = true;
				break;
			}
		}

		if (bSkip)
		{
			continue;
		}

		// Include this possessable since it does not belong to any other bindings that will be baked
		FinalizedBindings.Add(Possessable->GetGuid());
	}


	return FinalizedBindings;
}


bool FGeometryCacheLevelSequenceBaker::GetGeometryCacheAssetPathFromUser(FString& OutPackageName, FString& OutAssetName)
{
	FString NewNameSuggestion = FString(TEXT("NewGeometryCache"));
	FString DefaultPath;
	const FString DefaultDirectory = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::NEW_ASSET);
	FPackageName::TryConvertFilenameToLongPackageName(DefaultDirectory, DefaultPath);

	if (DefaultPath.IsEmpty())
	{
		DefaultPath = TEXT("/Game/GeometryCaches");
	}

	FString PackageNameSuggestion = DefaultPath / NewNameSuggestion;
	FString Name;
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().CreateUniqueAssetName(PackageNameSuggestion, TEXT(""), PackageNameSuggestion, Name);

	// Decide where to create the geo cache asset
	TSharedPtr<SDlgPickAssetPath> PickAssetPathWidget =
				SNew(SDlgPickAssetPath)
				.Title(LOCTEXT("BakeGeometryCachePickName", "Choose New Geometry Cache Location"))
				.DefaultAssetPath(FText::FromString(PackageNameSuggestion));

	if (PickAssetPathWidget->ShowModal() == EAppReturnType::Ok)
	{
		// Get the full name of where we want to create the mesh asset.
		OutPackageName = PickAssetPathWidget->GetFullAssetPath().ToString();
		OutAssetName = FPackageName::GetLongPackageAssetName(OutPackageName);

		// Check if the user inputed a valid asset name, if they did not, give it the generated default name
		if (OutAssetName.IsEmpty())
		{
			// Use the defaults that were already generated.
			OutPackageName = PackageNameSuggestion;
			OutAssetName = *Name;
		}

		return true;
	}

	return false;
}


void FGeometryCacheLevelSequenceBaker::Tick(float DeltaTime)
{
	if (CurrentBakeTask->Stage == EStage::Gather)
	{
		if (CurrentBakeTask->CurrentSampleIndex == 0)
		{
			CurrentBakeTask->PlaySequencer();
			return;
		}

		if (!CurrentBakeTask->IsSequencerPlaying())
		{
			if (CurrentBakeTask->CurrentSampleIndex < CurrentBakeTask->NumSamples)
			{
				return;
			}
			
			CurrentBakeTask->CurrentSampleIndex = 0;
			CurrentBakeTask->GatherStageComponentSettingScopes.Reset();
			SetupComponentBakeTasks();
		
			CurrentBakeTask->Stage = EStage::RequestReadback;	
		}
	}
		


	if (CurrentBakeTask->Stage == EStage::RequestReadback)
	{
		if (CurrentBakeTask->CurrentSampleIndex == 0)
		{
			CurrentBakeTask->PlaySequencer();
			return;
		}
		
		if (!CurrentBakeTask->IsSequencerPlaying())
		{
			if (CurrentBakeTask->CurrentSampleIndex < CurrentBakeTask->NumSamples)
			{
				return;
			}

			CurrentBakeTask->Stage = EStage::WriteToAsset;
		}
	}

	if (CurrentBakeTask->Stage == EStage::WriteToAsset)
	{
		if (!CurrentBakeTask->SlowTask->ShouldCancel())
		{
			if (CurrentBakeTask->NumComponentTasksPending > 0)
			{
				return;
			}
			
			WriteToAsset();
		}

		EndTask();
	}
}


void FGeometryCacheLevelSequenceBaker::OnWorldTickEnd(UWorld*, ELevelTick, float)
{
	if (CurrentBakeTask->Stage == EStage::Gather)
	{
		if (CurrentBakeTask->CurrentSampleIndex < CurrentBakeTask->NumSamples)
		{
			Gather();
			CurrentBakeTask->CurrentSampleIndex++;
		}
		return;
	}

	if (CurrentBakeTask->Stage == EStage::RequestReadback)
	{
		if (CurrentBakeTask->CurrentSampleIndex < CurrentBakeTask->NumSamples)
		{
			RequestReadback();	
			CurrentBakeTask->CurrentSampleIndex++;
		}

		return;
	}	
}

bool FGeometryCacheLevelSequenceBaker::IsTickable() const
{
	return CurrentBakeTask.IsValid();
}

FGeometryCacheLevelSequenceBaker& FGeometryCacheLevelSequenceBaker::Get()
{
	static FGeometryCacheLevelSequenceBaker Instance;
	return Instance;
}

void FGeometryCacheLevelSequenceBaker::SetupComponentBakeTasks()
{
	// Create readback tasks for each skeletal mesh component
	TArray<FComponentTask>& ComponentTasks = CurrentBakeTask->ComponentTasks;
	
	for (const TPair<FGuid, TMap<FName, FComponentInfo>>& BindingToComponentInfo : CurrentBakeTask->BindingToComponentInfoMap)
	{
		for (const TPair<FName, FComponentInfo>& ComponentInfo : BindingToComponentInfo.Value)
		{
			FComponentTask& ComponentTask = ComponentTasks.AddDefaulted_GetRef();
			ComponentTask.Binding = BindingToComponentInfo.Key;
			ComponentTask.ComponentInfo = ComponentInfo.Value;
			ComponentTask.NumSamplesPending = CurrentBakeTask->NumSamples;
			ComponentTask.GeometrySamples.Init({}, ComponentTask.NumSamplesPending);
			ComponentTask.VisibilitySamples.Init({}, ComponentTask.NumSamplesPending);
		}
	}
	
	CurrentBakeTask->NumComponentTasksPending = CurrentBakeTask->ComponentTasks.Num();	
}

void FGeometryCacheLevelSequenceBaker::Gather()
{
	TSharedPtr<ISequencer> Sequencer = CurrentBakeTask->Sequencer.Pin();
	IMovieScenePlayer* MovieScenePlayer = Sequencer.Get();
	FMovieSceneSequenceIDRef SequenceID = Sequencer->GetFocusedTemplateID();

	for (FGuid Binding : CurrentBakeTask->Bindings)
	{
		TMap<FName, FComponentInfo>& ComponentInfosRef = CurrentBakeTask->BindingToComponentInfoMap.FindOrAdd(Binding);
		TArrayView<TWeakObjectPtr<>> RuntimeObjects = MovieScenePlayer->FindBoundObjects(Binding, SequenceID);
		if (RuntimeObjects.Num() > 0)
		{
			// At the moment only support one object per binding
			TWeakObjectPtr<UObject> RuntimeObject = RuntimeObjects[0];

			TArray<USkeletalMeshComponent*> SkeletalMeshComponents;
			if (RuntimeObject.IsValid())
			{
				if (AActor* Actor = Cast<AActor>(RuntimeObject.Get()))
				{
					// Bake all the skeletal mesh components for this actor
					TArray<USkeletalMeshComponent*> ActorSkeletalMeshComponents;
					Actor->GetComponents(ActorSkeletalMeshComponents);
					for (USkeletalMeshComponent* SkeletalMeshComponent : ActorSkeletalMeshComponents)
					{
						SkeletalMeshComponents.Add(SkeletalMeshComponent);
					}
				}
				else if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(RuntimeObject.Get()))
				{
					SkeletalMeshComponents.Add(SkeletalMeshComponent);
				}
			}

			for (USkeletalMeshComponent* SkeletalMeshComponent : SkeletalMeshComponents)
			{
				if (USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset())
				{
					FName Name = SkeletalMeshComponent->GetFName();
					if (FComponentInfo* ComponentInfo = ComponentInfosRef.Find(Name))
					{
						// At the moment we don't support changing mesh assets
						if (!ensure(ComponentInfo->SkeletalMeshAsset == SkeletalMesh))
						{
							FNotificationInfo ErrorToast(
								LOCTEXT("FailToBakeGeometryCache_Title",
								        "Bake to Geometry Cache failed"));
							ErrorToast.ExpireDuration = 10.0f;
							ErrorToast.bFireAndForget = true;
							ErrorToast.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Error"));
							ErrorToast.SubText = LOCTEXT("FailToBakeGeometryCache_MismatchedMeshAsset",
							                             "Changing Skeletal Mesh Asset during Playback is not Supported");
							FSlateNotificationManager::Get().AddNotification(ErrorToast);

							// clear the task and abort
							EndTask();
							return;
						}
					}

					{
						FComponentInfo Info;
						Info.Name = Name;
						Info.SkeletalMeshAsset = SkeletalMesh;
						// We need to reference the material assets instead of the dynamic instances on the component to avoid GC failure when changing level
						for (UMaterialInterface* MaterialInterface : SkeletalMeshComponent->GetMaterials())
						{
							UMaterialInterface* MaterialAsset = MaterialInterface;
							if (UMaterialInstanceDynamic* AsDynamic = Cast<UMaterialInstanceDynamic>(MaterialInterface))
							{
								MaterialAsset = AsDynamic->Parent;
							}

							check(MaterialAsset == nullptr || MaterialAsset->IsAsset());
							
							Info.Materials.Add(MaterialAsset);
						}
						ComponentInfosRef.Add(Name, MoveTemp(Info));
					}

					// Try to test-run deformers on these skeletal mesh component to make sure deformer shaders are compiled and ready
					TUniquePtr<FSkeletalMeshComponentSettingScope>& ComponentSettingScope = CurrentBakeTask->GatherStageComponentSettingScopes.
						FindOrAdd(SkeletalMeshComponent);
					if (!ComponentSettingScope.IsValid())
					{
						ComponentSettingScope = MakeUnique<FSkeletalMeshComponentSettingScope>(SkeletalMeshComponent);
					}
				}
			}
		}
	}

	CurrentBakeTask->UpdateGatherProgress();
}

void FGeometryCacheLevelSequenceBaker::RequestReadback()
{
	TSharedPtr<ISequencer> Sequencer = CurrentBakeTask->Sequencer.Pin();
	IMovieScenePlayer* MovieScenePlayer = Sequencer.Get();
	FMovieSceneSequenceIDRef SequenceID = Sequencer->GetFocusedTemplateID();

	for (int32 TaskIndex = 0;  TaskIndex < CurrentBakeTask->ComponentTasks.Num(); ++TaskIndex)
	{
		FComponentTask& ComponentTask = CurrentBakeTask->ComponentTasks[TaskIndex];
		
		bool bReadbackRequested = false;

		USkeletalMeshComponent* Component = nullptr;
		
		TArrayView<TWeakObjectPtr<>> RuntimeObjects = MovieScenePlayer->FindBoundObjects(ComponentTask.Binding, SequenceID);
		if (RuntimeObjects.Num() > 0 && RuntimeObjects[0].IsValid())
		{
			// At the moment only support one object per binding
			TWeakObjectPtr<UObject> RuntimeObject = RuntimeObjects[0];

			if (AActor* Actor = Cast<AActor>(RuntimeObject.Get()))
			{
				// Bake all the skeletal mesh components for this actor
				TArray<USkeletalMeshComponent*> ActorSkeletalMeshComponents;
				Actor->GetComponents(ActorSkeletalMeshComponents);

				for (USkeletalMeshComponent* SkeletalMeshComponent : ActorSkeletalMeshComponents)
				{
					if (SkeletalMeshComponent->GetFName() == ComponentTask.ComponentInfo.Name)
					{
						Component = SkeletalMeshComponent;
					}
				}
			}
			else if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(RuntimeObject.Get()))
			{
				if (SkeletalMeshComponent->GetFName() == ComponentTask.ComponentInfo.Name)
				{
					Component = SkeletalMeshComponent;
				}
			}
		}

		if (Component)
		{
			if (ComponentTask.ComponentSettingScope == nullptr || ComponentTask.ComponentSettingScope->Component != Component)
			{
				ComponentTask.ComponentSettingScope = MakeUnique<FSkeletalMeshComponentSettingScope>(Component);
			}

			FTransform Transform = Component->GetComponentTransform();
			TUniquePtr<FMeshDeformerGeometryReadbackRequest> Request = MakeUnique<FMeshDeformerGeometryReadbackRequest>();

			Request->VertexDataArraysCallback_AnyThread =
				[
					Transform,
					SampleIndex = CurrentBakeTask->CurrentSampleIndex,
					TaskIndex
				](const FMeshDeformerGeometryReadbackVertexDataArrays& VertexDataArrays)
				{
					// Making sure current bake task is alive when this callback is invoked on a worker thread
					FScopeLock BakeTaskLifeTimeLock(&Get().CurrentBakeTaskLifeTimeCriticalSection);
					TUniquePtr<FBakeTask>& BakeTask = Get().CurrentBakeTask; 
					if (BakeTask == nullptr)
					{
						return;
					}

					int32 NumVertices = VertexDataArrays.Positions.Num();
					
					FComponentTask& ComponentTask = BakeTask->ComponentTasks[TaskIndex];
					FFrameData& FrameData = ComponentTask.GeometrySamples[SampleIndex];
					
					bool bMeshAvailable = true;
					if (VertexDataArrays.LODIndex == INDEX_NONE || NumVertices == 0)
					{
						bMeshAvailable = false;
					}
					else if (ComponentTask.ActualLODIndexBaked != INDEX_NONE && ComponentTask.ActualLODIndexBaked != VertexDataArrays.LODIndex) // LOD changed during bake, not supported
					{
						bMeshAvailable = false;
					}
					
					if (bMeshAvailable)
					{
						 if (ComponentTask.ActualLODIndexBaked == INDEX_NONE)
						{
							ComponentTask.ActualLODIndexBaked = VertexDataArrays.LODIndex;
						}
						
						FrameData.Positions.Init({}, NumVertices);
						FrameData.Normals.Init({}, NumVertices);
						FrameData.TangentsX.Init({}, NumVertices);
						
						for (int32 Index = 0; Index < NumVertices; Index++)
						{
							FrameData.Positions[Index] = FVector3f(Transform.TransformPosition(FVector(VertexDataArrays.Positions[Index])));
							if (VertexDataArrays.Normals.IsValidIndex(Index))
							{
								FrameData.Normals[Index] = FVector3f(Transform.TransformVector(FVector(VertexDataArrays.Normals[Index])));
							}
							if (VertexDataArrays.Tangents.IsValidIndex(Index))
							{
								FrameData.TangentsX[Index] = FVector3f(Transform.TransformVector(FVector(VertexDataArrays.Tangents[Index])));
							}
						}
					}

					OnReadbackResultConfirmed(ComponentTask, SampleIndex, bMeshAvailable);	
				};
				
			Component->RequestReadbackRenderGeometry(MoveTemp(Request));
			
			bReadbackRequested = true;
		}

		if (!bReadbackRequested)
		{
			constexpr bool bMeshAvailable = false;
			OnReadbackResultConfirmed(ComponentTask, CurrentBakeTask->CurrentSampleIndex, bMeshAvailable);		
		}
	}
}

void FGeometryCacheLevelSequenceBaker::WriteToAsset()
{
	UPackage* Package = CreatePackage(*CurrentBakeTask->PackageName);
	check(Package);
	UGeometryCache* GeometryCache = NewObject<UGeometryCache>(Package, *CurrentBakeTask->AssetName, RF_Public | RF_Standalone);
	
	using UE::GeometryCacheHelpers::FGeometryCacheConstantTopologyWriter;
	using UE::GeometryCacheHelpers::AddTrackWriterFromSkinnedAsset;
	using FTrackWriter = FGeometryCacheConstantTopologyWriter::FTrackWriter;
	FGeometryCacheConstantTopologyWriter::FConfig Config;
	Config.FPS = CurrentBakeTask->SamplesPerSecond;
	FGeometryCacheConstantTopologyWriter Writer(*GeometryCache, Config);

	for (FComponentTask& Task : CurrentBakeTask->ComponentTasks)
	{

		int32 FirstVisibleFrameIndex = INDEX_NONE;
		for (const FVisibilitySample& VisibilitySample : Task.VisibilitySamples)
		{
			if (VisibilitySample.bVisible && FirstVisibleFrameIndex == INDEX_NONE)
			{
				FirstVisibleFrameIndex = VisibilitySample.FrameIndex;
			}
		}

		if (FirstVisibleFrameIndex == INDEX_NONE)
		{
			continue;
		}

		int32 NumVertices = Task.GeometrySamples[FirstVisibleFrameIndex].Positions.Num();
		
		// Copy neighbor frames when visibility is changing such that frame interpolation works well
		for (const FVisibilitySample& VisibilitySample : Task.VisibilitySamples)
		{	
			if (!VisibilitySample.bVisible)
			{
				const int32 Previous = VisibilitySample.FrameIndex-1 ;
				const int32 Next = VisibilitySample.FrameIndex+1 ;
				// Copy from next if changing to visible
				if (Task.VisibilitySamples.IsValidIndex(Next) &&
					Task.VisibilitySamples[Next].bVisible)
				{
					Task.GeometrySamples[VisibilitySample.FrameIndex] = Task.GeometrySamples[Next];	
				}
				// Copy from previous if changing to invisible
				else if (Task.VisibilitySamples.IsValidIndex(Previous) && 
					Task.VisibilitySamples[Previous].bVisible)
				{
					Task.GeometrySamples[VisibilitySample.FrameIndex] = Task.GeometrySamples[Previous];	
				}
			}
		}

		// Make sure the first frame has valid data, even if it is invisible, so that the geo cache preprocessor can use it to for sanity checks
		if (!Task.VisibilitySamples[0].bVisible)
		{
			Task.GeometrySamples[0] = Task.GeometrySamples[Task.VisibilitySamples[FirstVisibleFrameIndex].FrameIndex];
		}

		for (int32 Frame = 0; Frame < Task.GeometrySamples.Num(); ++Frame)
		{
			FFrameData& FrameData = Task.GeometrySamples[Frame];
			
			if (FrameData.Positions.Num() != Task.GeometrySamples[0].Positions.Num())
			{
				if (ensure(FrameData.Positions.Num() == 0))
				{
					FrameData.Positions.Init({}, NumVertices);
					FrameData.Normals.Init({}, NumVertices);
					FrameData.TangentsX.Init({}, NumVertices);	
				}
			}
		}

		int32 TrackIndex = AddTrackWriterFromSkinnedAssetAndMaterials(Writer, *(Task.ComponentInfo.SkeletalMeshAsset), Task.ActualLODIndexBaked, Task.ComponentInfo.Materials);

		FTrackWriter& TrackWriter = Writer.GetTrackWriter(TrackIndex);

		TrackWriter.WriteAndClose(Task.GeometrySamples, Task.VisibilitySamples);
		Task.GeometrySamples.Reset();
	}

	GeometryCache->MarkPackageDirty();

	// Notify asset registry of new asset
	FAssetRegistryModule::AssetCreated(GeometryCache);

	// Display notification so users can quickly access the mesh
	if (GIsEditor)
	{
		FNotificationInfo Info(FText::Format(
			LOCTEXT("GeometryCacheBaked", "Successfully Baked to Geometry Cache"), FText::FromString(GeometryCache->GetName())));
		Info.ExpireDuration = 15.0f;
		Info.bUseLargeFont = false;
		Info.Hyperlink = FSimpleDelegate::CreateLambda([=]()
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAssets(TArray<UObject*>({GeometryCache}));
		});
		Info.HyperlinkText = FText::Format(LOCTEXT("OpenNewGeometryCacheHyperlink", "Open {0}"), FText::FromString(GeometryCache->GetName()));
		TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
		if (Notification.IsValid())
		{
			Notification->SetCompletionState(SNotificationItem::CS_Success);
		}
	}
}

FGeometryCacheLevelSequenceBaker::FSkeletalMeshComponentSettingScope::FSkeletalMeshComponentSettingScope(USkeletalMeshComponent* InComponent)
	: Component(InComponent)
{

	bPreviousAlwaysUseMeshDeformer = Component->GetAlwaysUseMeshDeformer();
	Component->SetAlwaysUseMeshDeformer(true);

	bPreviousUpdateAnimationInEditor = Component->GetUpdateAnimationInEditor();
	Component->SetUpdateAnimationInEditor(true);

	PreviousForcedLOD = Component->GetForcedLOD();
	Component->SetForcedLOD(LODIndexToBake+1);
}

FGeometryCacheLevelSequenceBaker::FSkeletalMeshComponentSettingScope::~FSkeletalMeshComponentSettingScope()
{
	if (Component.IsValid())
	{
		Component->SetAlwaysUseMeshDeformer(bPreviousAlwaysUseMeshDeformer);
		Component->SetUpdateAnimationInEditor(bPreviousUpdateAnimationInEditor);
		Component->SetForcedLOD(PreviousForcedLOD);
	}
}

void FGeometryCacheLevelSequenceBaker::OnReadbackResultConfirmed(FComponentTask& ComponentTask, int32 SampleIndex, bool bMeshAvailable)
{
	ComponentTask.VisibilitySamples[SampleIndex].FrameIndex = SampleIndex;
	ComponentTask.VisibilitySamples[SampleIndex].bVisible = bMeshAvailable;

	if (IsInGameThread())
	{
		Get().CurrentBakeTask->UpdateBakeProgress();
	}
	else
	{
		FFunctionGraphTask::CreateAndDispatchWhenReady([]()
		{
			if (Get().CurrentBakeTask)
			{
				Get().CurrentBakeTask->UpdateBakeProgress();
			}
		}, TStatId(), NULL, ENamedThreads::GameThread);
	}

	if (--ComponentTask.NumSamplesPending == 0)
	{
		--Get().CurrentBakeTask->NumComponentTasksPending;
	}
}

FGeometryCacheLevelSequenceBaker::FEngineFixedDeltaTimeScope::FEngineFixedDeltaTimeScope(double NewFixedDeltaTime)
{
	bPreviousUseFixedDeltaTime = FApp::UseFixedTimeStep();
	FApp::SetUseFixedTimeStep(true);
	
	PreviousFixedDeltaTime = FApp::GetFixedDeltaTime();
	FApp::SetFixedDeltaTime(NewFixedDeltaTime);
}

FGeometryCacheLevelSequenceBaker::FEngineFixedDeltaTimeScope::~FEngineFixedDeltaTimeScope()
{
	FApp::SetUseFixedTimeStep(bPreviousUseFixedDeltaTime);
	FApp::SetFixedDeltaTime(PreviousFixedDeltaTime);
}

FGeometryCacheLevelSequenceBaker::FSequencerSettingScope::FSequencerSettingScope(TSharedPtr<ISequencer> InSequencer)
{
	Sequencer = InSequencer;
	PreviousLocalTime = InSequencer->GetLocalTime();
	PreviousLoopMode = InSequencer->GetSequencerSettings()->GetLoopMode();
	InSequencer->GetSequencerSettings()->SetLoopMode(ESequencerLoopMode::SLM_NoLoop);

	UMovieSceneSequence* MovieSceneSequence = InSequencer->GetFocusedMovieSceneSequence();
	UMovieScene* MovieScene = MovieSceneSequence->GetMovieScene();

	// ensure 1 sample per tick
	PreviousClockSource = MovieScene->GetClockSource();
	MovieScene->SetClockSource(EUpdateClockSource::Tick);
	InSequencer->ResetTimeController();
}

FGeometryCacheLevelSequenceBaker::FSequencerSettingScope::~FSequencerSettingScope()
{
	if (ensure(Sequencer.Pin()))
	{
		UMovieSceneSequence* MovieSceneSequence = Sequencer.Pin()->GetFocusedMovieSceneSequence();
		UMovieScene* MovieScene = MovieSceneSequence->GetMovieScene();
		MovieScene->SetClockSource(PreviousClockSource);
		Sequencer.Pin()->ResetTimeController();
		
		Sequencer.Pin()->GetSequencerSettings()->SetLoopMode(PreviousLoopMode);
		Sequencer.Pin()->SetLocalTime(PreviousLocalTime.Time);
	}
}

FGeometryCacheLevelSequenceBaker::FConsoleVariableOverrideScope::FConsoleVariableOverrideScope()
{
	// Needed for metahuman which uses LODSyncComponent
	if (IConsoleVariable* CVarForceLOD = IConsoleManager::Get().FindConsoleVariable(TEXT("r.ForceLOD")))
	{
		PreviousForceLOD = CVarForceLOD->GetInt();
		CVarForceLOD->Set(0);
	}


	if (IConsoleVariable* CVarSkeletalMeshLODBias = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SkeletalMeshLODBias")))
	{
		PreviousSkeletalMeshLODBias = CVarSkeletalMeshLODBias->GetInt();
		CVarSkeletalMeshLODBias->Set(-10);
	}
}

FGeometryCacheLevelSequenceBaker::FConsoleVariableOverrideScope::~FConsoleVariableOverrideScope()
{
	if (IConsoleVariable* CVarForceLOD = IConsoleManager::Get().FindConsoleVariable(TEXT("r.ForceLOD")))
	{
		CVarForceLOD->Set(PreviousForceLOD);
	}

	if (IConsoleVariable* CVarSkeletalMeshLODBias = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SkeletalMeshLODBias")))
	{
		CVarSkeletalMeshLODBias->Set(PreviousSkeletalMeshLODBias);
	}
}

FGeometryCacheLevelSequenceBaker::FWorldTickEndDelegateScope::FWorldTickEndDelegateScope()
{
	OnWorldTickEndDelegate = FWorldDelegates::OnWorldTickEnd.AddRaw(&Get(), &FGeometryCacheLevelSequenceBaker::OnWorldTickEnd);
}

FGeometryCacheLevelSequenceBaker::FWorldTickEndDelegateScope::~FWorldTickEndDelegateScope()
{
	FWorldDelegates::OnWorldTickEnd.Remove(OnWorldTickEndDelegate);
}

bool FGeometryCacheLevelSequenceBaker::FBakeTask::IsSequencerPlaying()
{
	return Sequencer.Pin()->GetPlaybackStatus() == EMovieScenePlayerStatus::Playing;
}

void FGeometryCacheLevelSequenceBaker::FBakeTask::PlaySequencer()
{
	Sequencer.Pin()->SetLocalTime(StartFrame);
	Sequencer.Pin()->PlayTo({EndFrame, EUpdatePositionMethod::Play});
}

void FGeometryCacheLevelSequenceBaker::FBakeTask::UpdateGatherProgress()
{
	SlowTask->EnterProgressFrame(AmountOfWorkGatherStage/NumSamples);
}

void FGeometryCacheLevelSequenceBaker::FBakeTask::UpdateBakeProgress()
{
	SlowTask->EnterProgressFrame(AmountOfWorkBakeStage/(NumSamples * ComponentTasks.Num()));
}

void FGeometryCacheLevelSequenceBaker::FBakeTask::TickProgress()
{
	SlowTask->TickProgress();
}

void FGeometryCacheLevelSequenceBaker::EndTask()
{
	// Make sure to wait for ongoing readback callbacks to finish
	FScopeLock BakeTaskLifeTimeLock(&CurrentBakeTaskLifeTimeCriticalSection);
	CurrentBakeTask = nullptr;
}

#undef LOCTEXT_NAMESPACE
