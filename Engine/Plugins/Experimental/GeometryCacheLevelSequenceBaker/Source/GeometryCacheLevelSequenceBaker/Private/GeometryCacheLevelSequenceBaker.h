// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryCacheConstantTopologyWriter.h"
#include "ISequencer.h"
#include "SequencerSettings.h"
#include "Widgets/SCompoundWidget.h"
#include "TickableEditorObject.h"
#include "Misc/ScopedSlowTask.h"
#include "Engine/SkeletalMesh.h"

#include "GeometryCacheLevelSequenceBaker.generated.h"

class UGeometryCache;
class UMaterialInterface;

UCLASS()
class ULevelSequenceGeometryCacheBakerOption : public UObject
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = Options)
	int32 NumSamplesPerFrame = 1;
	
	UPROPERTY()
	bool bShouldBake = false;
};

class USkeletalMeshComponent;

class GEOMETRYCACHELEVELSEQUENCEBAKER_API FGeometryCacheLevelSequenceBaker : public FTickableEditorObject
{
public:

	static void Bake(TSharedRef<ISequencer> InSequencer );
	static TArray<FGuid> GetBindingsToBake(TSharedRef<ISequencer> InSequencer);
	static bool GetGeometryCacheAssetPathFromUser(FString& OutPackageName, FString& OutAssetName);

	void Tick(float DeltaTime) override;
	void OnWorldTickEnd(UWorld*, ELevelTick, float);
	bool IsTickable() const override;
	TStatId GetStatId() const override {RETURN_QUICK_DECLARE_CYCLE_STAT(FBlueprintActionDatabase, STATGROUP_Tickables);}
	
private:
	FGeometryCacheLevelSequenceBaker() = default;
	static FGeometryCacheLevelSequenceBaker& Get();

	void SetupComponentBakeTasks();
	void Gather();
	void RequestReadback();
	void WriteToAsset();

	

	static const int32 LODIndexToBake;
	static const float TotalAmountOfWork;
	static const float AmountOfWorkGatherStage;
	static const float AmountOfWorkBakeStage;
	
	struct FSkeletalMeshComponentSettingScope
	{
		TWeakObjectPtr<USkeletalMeshComponent> Component;
		
		bool bPreviousAlwaysUseMeshDeformer = false;
		bool bPreviousUpdateAnimationInEditor = false;
		int32 PreviousForcedLOD = 0;
		
		FSkeletalMeshComponentSettingScope(USkeletalMeshComponent* InComponent);
		~FSkeletalMeshComponentSettingScope();
	};

	using FFrameData = UE::GeometryCacheHelpers::FGeometryCacheConstantTopologyWriter::FFrameData;
	using FVisibilitySample = UE::GeometryCacheHelpers::FGeometryCacheConstantTopologyWriter::FVisibilitySample;

	struct FComponentInfo
	{
		FName Name;
		USkeletalMesh* SkeletalMeshAsset;
		TArray<TObjectPtr<UMaterialInterface>> Materials;
	};
	
	struct FComponentTask
	{
		FGuid Binding;

		FComponentInfo ComponentInfo;

		int32 ActualLODIndexBaked = INDEX_NONE;
		TArray<FFrameData> GeometrySamples;
		std::atomic<uint32> NumSamplesPending = 0;
		TArray<FVisibilitySample> VisibilitySamples;

		TUniquePtr<FSkeletalMeshComponentSettingScope> ComponentSettingScope;
	};

	static void OnReadbackResultConfirmed(FComponentTask& Task, int32 SampleIndex, bool bMeshAvailable);

	struct FEngineFixedDeltaTimeScope
	{
		bool bPreviousUseFixedDeltaTime = false;
		double PreviousFixedDeltaTime = 0;
		FEngineFixedDeltaTimeScope(double NewFixedDeltaTime);
		~FEngineFixedDeltaTimeScope();
	};

	struct FSequencerSettingScope
	{
		TWeakPtr<ISequencer> Sequencer;
		FQualifiedFrameTime PreviousLocalTime;
		ESequencerLoopMode PreviousLoopMode;
		EUpdateClockSource PreviousClockSource;
		
		FSequencerSettingScope(TSharedPtr<ISequencer> InSequencer);
		~FSequencerSettingScope();
	};

	struct FConsoleVariableOverrideScope
	{
		int32 PreviousForceLOD = 0;
		int32 PreviousSkeletalMeshLODBias = 0;
		FConsoleVariableOverrideScope();
		~FConsoleVariableOverrideScope();
	};
	
	struct FWorldTickEndDelegateScope
	{
		FDelegateHandle OnWorldTickEndDelegate;
		FWorldTickEndDelegateScope();
		~FWorldTickEndDelegateScope();
	};

	enum class EStage
	{
		Gather,
		RequestReadback,
		WriteToAsset,
		End
	};


	
	struct FBakeTask
	{
		FString PackageName;
		FString AssetName;

		TWeakPtr<ISequencer> Sequencer;
		TArray<FGuid> Bindings;
		TMap<FGuid, TMap<FName, FComponentInfo>> BindingToComponentInfoMap;
		FFrameTime StartFrame;
		FFrameTime EndFrame;
		
		EStage Stage = EStage::Gather;
		
		TArray<FComponentTask> ComponentTasks;
		std::atomic<uint32> NumComponentTasksPending = 0;
		
		uint32 NumSamples = 0;
		uint32 CurrentSampleIndex = 0;
		float SamplesPerSecond = 0;

		TUniquePtr<FScopedSlowTask> SlowTask;
		TUniquePtr<FEngineFixedDeltaTimeScope> FixedDeltaTimeScope;
		TUniquePtr<FSequencerSettingScope> SequencerStateScope;
		TUniquePtr<FConsoleVariableOverrideScope> ConsoleVariableOverrideScope;
		TUniquePtr<FWorldTickEndDelegateScope> WorldTickEndDelegateScope;

		FDelegateHandle OnEndFrameDelegateHandle;

		TMap<TWeakObjectPtr<USkeletalMeshComponent>, TUniquePtr<FSkeletalMeshComponentSettingScope>> GatherStageComponentSettingScopes;

		bool IsSequencerPlaying();
		void PlaySequencer();
		
		void UpdateGatherProgress();
		void UpdateBakeProgress();
		void TickProgress();
	};

	void EndTask();

	FCriticalSection CurrentBakeTaskLifeTimeCriticalSection;
	TUniquePtr<FBakeTask> CurrentBakeTask;
};




