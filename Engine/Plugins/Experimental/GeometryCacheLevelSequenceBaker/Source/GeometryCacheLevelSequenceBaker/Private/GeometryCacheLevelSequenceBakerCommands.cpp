// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheLevelSequenceBakerCommands.h"
#include "GeometryCacheLevelSequenceBakerStyle.h"

#define LOCTEXT_NAMESPACE "FGeometryCacheLevelSequenceBakerCommands"

FGeometryCacheLevelSequenceBakerCommands::FGeometryCacheLevelSequenceBakerCommands()
	: TCommands<FGeometryCacheLevelSequenceBakerCommands>(
		"GeometryCacheLevelSequenceBakerCommands", // Context name for fast lookup
		NSLOCTEXT("Contexts", "GeometryCacheLevelSequenceBakerCommands", "Geometry Cache Level Sequence Baker"), // Localized context name for displaying
		NAME_None, // Parent
		FGeometryCacheLevelSequenceBakerStyle::Get().GetStyleSetName() // Icon Style Set
		)
{
}

void FGeometryCacheLevelSequenceBakerCommands::RegisterCommands()
{
	UI_COMMAND(BakeGeometryCache, "Bake Geometry Cache", "Bake the selected skeletal meshes (all if none selected) to a Geometry Cache. (Shots and sub-scenes not supported)", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE

