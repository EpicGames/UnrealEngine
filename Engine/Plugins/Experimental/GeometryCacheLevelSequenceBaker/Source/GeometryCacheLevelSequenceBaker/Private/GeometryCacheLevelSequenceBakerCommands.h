// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"

class FGeometryCacheLevelSequenceBakerCommands : public TCommands<FGeometryCacheLevelSequenceBakerCommands>
{
public:
	FGeometryCacheLevelSequenceBakerCommands();

	/** Exports sequence to geometry cache. */
	TSharedPtr< FUICommandInfo > BakeGeometryCache;
	
	virtual void RegisterCommands() override;
};
