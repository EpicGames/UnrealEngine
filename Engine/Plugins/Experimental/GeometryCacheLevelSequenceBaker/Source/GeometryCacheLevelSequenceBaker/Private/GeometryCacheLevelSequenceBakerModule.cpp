// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheLevelSequenceBakerModule.h"

#include "FGeometryCacheLevelSequenceBakerCustomization.h"
#include "GeometryCacheLevelSequenceBakerCommands.h"
#include "GeometryCacheLevelSequenceBakerStyle.h"
#include "LevelSequenceEditorModule.h"
#include "SequencerCustomizationManager.h"

DEFINE_LOG_CATEGORY(LogGeometryCacheLevelSequenceBaker);

#define LOCTEXT_NAMESPACE "FGeometryCacheLevelSequenceBakerModule"

void FGeometryCacheLevelSequenceBakerModule::StartupModule()
{
	FGeometryCacheLevelSequenceBakerStyle::Register();	
	FGeometryCacheLevelSequenceBakerCommands::Register();
	
	ILevelSequenceEditorModule& LevelSequenceEditorModule = FModuleManager::Get().LoadModuleChecked<ILevelSequenceEditorModule>("LevelSequenceEditor");
	CustomizationHandle = LevelSequenceEditorModule.RegisterAdditionalLevelSequenceEditorCustomization(FOnGetSequencerCustomizationInstance::CreateLambda([]()
	{
		return new FGeometryCacheLevelSequenceBakerCustomization();
	}));
}

void FGeometryCacheLevelSequenceBakerModule::ShutdownModule()
{
	ILevelSequenceEditorModule& LevelSequenceEditorModule = FModuleManager::Get().LoadModuleChecked<ILevelSequenceEditorModule>("LevelSequenceEditor");
	LevelSequenceEditorModule.UnregisterAdditionalLevelSequenceEditorCustomization(CustomizationHandle);
}





#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FGeometryCacheLevelSequenceBakerModule, GeometryCacheLevelSequenceBaker)