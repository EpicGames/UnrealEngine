// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheLevelSequenceBakerStyle.h"

#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"


FGeometryCacheLevelSequenceBakerStyle& FGeometryCacheLevelSequenceBakerStyle::Get()
{
	static FGeometryCacheLevelSequenceBakerStyle Instance;
	return Instance;
}

void FGeometryCacheLevelSequenceBakerStyle::Register()
{
	FSlateStyleRegistry::RegisterSlateStyle(Get());
}

void FGeometryCacheLevelSequenceBakerStyle::Unregister()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(Get());
}

FGeometryCacheLevelSequenceBakerStyle::FGeometryCacheLevelSequenceBakerStyle()
	: FSlateStyleSet("GeometryCacheLevelSequenceBakerStyle")
{
	static const FVector2D IconSize10x10(10.0f, 10.0f);
	static const FVector2D IconSize16x12(16.0f, 12.0f);
	static const FVector2D IconSize16x16(16.0f, 16.0f);
	static const FVector2D IconSize20x20(20.0f, 20.0f);
	static const FVector2D IconSize64x64(64.0f, 64.0f);

	static const FSlateColor DefaultForeground(FLinearColor(0.72f, 0.72f, 0.72f, 1.f));

	FSlateStyleSet::SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Experimental/GeometryCacheLevelSequenceBaker/Resources"));
	{
		Set("GeometryCacheLevelSequenceBakerCommands.BakeGeometryCache", new IMAGE_BRUSH("Icon128", IconSize20x20));
	}
}
