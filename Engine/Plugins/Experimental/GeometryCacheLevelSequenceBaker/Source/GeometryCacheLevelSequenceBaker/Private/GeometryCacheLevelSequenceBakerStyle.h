// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"


class FGeometryCacheLevelSequenceBakerStyle
	: public FSlateStyleSet
{
public:
	static FGeometryCacheLevelSequenceBakerStyle& Get();

protected:
	friend class FGeometryCacheLevelSequenceBakerModule;

	static void Register();
	static void Unregister();

private:
	FGeometryCacheLevelSequenceBakerStyle();
};