// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "Misc/Guid.h"

DECLARE_LOG_CATEGORY_EXTERN(LogGeometryCacheLevelSequenceBaker, Log, All);

class FGeometryCacheLevelSequenceBakerModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	FGuid CustomizationHandle;
};
